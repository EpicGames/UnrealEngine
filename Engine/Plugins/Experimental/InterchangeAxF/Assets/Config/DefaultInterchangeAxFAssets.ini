[CoreRedirects]

; Redirectors for /Interchange/Functions/ (30)
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/DefaultSpecularProfile.DefaultSpecularProfile", NewName="/InterchangeAxFAssets/BaseMaterials/DefaultSpecularProfile.DefaultSpecularProfile")
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/SVBRDF.SVBRDF", NewName="/InterchangeAxFAssets/BaseMaterials/SVBRDF.SVBRDF")
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/SVBRDF_Alpha.SVBRDF_Alpha", NewName="/InterchangeAxFAssets/BaseMaterials/SVBRDF_Alpha.SVBRDF_Alpha")
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/SVBRDF_Alpha_POM.SVBRDF_Alpha_POM", NewName="/InterchangeAxFAssets/BaseMaterials/SVBRDF_Alpha_POM.SVBRDF_Alpha_POM")
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/SVBRDF_POM.SVBRDF_POM", NewName="/InterchangeAxFAssets/BaseMaterials/SVBRDF_POM.SVBRDF_POM")
+PackageRedirects=(OldName="/InterchangeAxF/BaseMaterials/CPA2.CPA2", NewName="/InterchangeAxFAssets/BaseMaterials/CPA2.CPA2")

+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_InvertMultipleScatteringGGX.MF_InvertMultipleScatteringGGX", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_InvertMultipleScatteringGGX.MF_InvertMultipleScatteringGGX")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_MakeTriPlanarUV.MF_MakeTriPlanarUV", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_MakeTriPlanarUV.MF_MakeTriPlanarUV")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_Noise_InterLeavedGradientGloden_1d.MF_Noise_InterLeavedGradientGloden_1d", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_Noise_InterLeavedGradientGloden_1d.MF_Noise_InterLeavedGradientGloden_1d")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_NormalStrength.MF_NormalStrength", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_NormalStrength.MF_NormalStrength")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_POM_TC.MF_POM_TC", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_POM_TC.MF_POM_TC")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_POM_TP.MF_POM_TP", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_POM_TP.MF_POM_TP")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_POMOverrideBlend.MF_POMOverrideBlend", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_POMOverrideBlend.MF_POMOverrideBlend")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_Rotate2D.MF_Rotate2D", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_Rotate2D.MF_Rotate2D")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_SampleNormalTEX.MF_SampleNormalTEX", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_SampleNormalTEX.MF_SampleNormalTEX")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_SampleTEX.MF_SampleTEX", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_SampleTEX.MF_SampleTEX")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_SheenRoughness.MF_SheenRoughness", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_SheenRoughness.MF_SheenRoughness")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_smin.MF_smin", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_smin.MF_smin")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_TexCoodUVNormalSampler.MF_TexCoodUVNormalSampler", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_TexCoodUVNormalSampler.MF_TexCoodUVNormalSampler")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_TextureCoordinationTransform.MF_TextureCoordinationTransform", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_TextureCoordinationTransform.MF_TextureCoordinationTransform")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_Triplanar_Core.MF_Triplanar_Core", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_Triplanar_Core.MF_Triplanar_Core")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_Triplanar_NormalCorrection.MF_Triplanar_NormalCorrection", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_Triplanar_NormalCorrection.MF_Triplanar_NormalCorrection")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_TriplanarBlendHeightA.MF_TriplanarBlendHeightA", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_TriplanarBlendHeightA.MF_TriplanarBlendHeightA")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_TriPlanarUV_Coods.MF_TriPlanarUV_Coods", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_TriPlanarUV_Coods.MF_TriPlanarUV_Coods")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_TriPlanarUVNormalSampler.MF_TriPlanarUVNormalSampler", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_TriPlanarUVNormalSampler.MF_TriPlanarUVNormalSampler")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_ViewTaceLength_TP.MF_ViewTaceLength_TP", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_ViewTaceLength_TP.MF_ViewTaceLength_TP")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_ViewTraceLength_TC.MF_ViewTraceLength_TC", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_ViewTraceLength_TC.MF_ViewTraceLength_TC")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/SheenRoughnessComponent.SheenRoughnessComponent", NewName="/InterchangeAxFAssets/MaterialFunctions/SheenRoughnessComponent.SheenRoughnessComponent")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/TextureCoordinateTransform.TextureCoordinateTransform", NewName="/InterchangeAxFAssets/MaterialFunctions/TextureCoordinateTransform.TextureCoordinateTransform")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/AnisotropyComponent.AnisotropyComponent", NewName="/InterchangeAxFAssets/MaterialFunctions/AnisotropyComponent.AnisotropyComponent")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/InvertMultipleScatteringGGX.InvertMultipleScatteringGGX", NewName="/InterchangeAxFAssets/MaterialFunctions/InvertMultipleScatteringGGX.InvertMultipleScatteringGGX")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_AnisotropyComponent.MF_AnisotropyComponent", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_AnisotropyComponent.MF_AnisotropyComponent")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_AnisotropyTangent.MF_AnisotropyTangent", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_AnisotropyTangent.MF_AnisotropyTangent")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_ClearCoatTerm.MF_ClearCoatTerm", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_ClearCoatTerm.MF_ClearCoatTerm")
+PackageRedirects=(OldName="/InterchangeAxF/MaterialFunctions/MF_DefaultBlend.MF_DefaultBlend", NewName="/InterchangeAxFAssets/MaterialFunctions/MF_DefaultBlend.MF_DefaultBlend")

+PackageRedirects=(OldName="/InterchangeAxF/Meshes/SM_MatPreviewMesh_01.SM_MatPreviewMesh_01", NewName="/InterchangeAxFAssets/Meshes/SM_MatPreviewMesh_01.SM_MatPreviewMesh_01")

+PackageRedirects=(OldName="/InterchangeAxF/Textures/GGX_ReflectionEnergy.GGX_ReflectionEnergy", NewName="/InterchangeAxFAssets/Textures/GGX_ReflectionEnergy.GGX_ReflectionEnergy")