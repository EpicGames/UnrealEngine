{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange AxF Assets",
	"Description": "The Interchange AxF Assets plugin exposes the assets used by the InterchangeAxf import plugin.",
	"Category": "",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "InterchangeAxFAssets",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "InterchangeAssets",
			"Enabled": true
		},
		{
			"Name": "BaseMaterial",
			"Enabled": true
		}
	]
}