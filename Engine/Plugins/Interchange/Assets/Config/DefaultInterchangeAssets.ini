[CoreRedirects]

; Redirectors for /Interchange/Functions/ (30)
+PackageRedirects=(OldName="/Interchange/Functions/MF_Iridescence", NewName="/InterchangeAssets/Functions/MF_Iridescence")
+PackageRedirects=(OldName="/Interchange/Functions/MF_OrenNayerView", NewName="/InterchangeAssets/Functions/MF_OrenNayerView")
+PackageRedirects=(OldName="/Interchange/Functions/MF_PhongToMetalRoughness", NewName="/InterchangeAssets/Functions/MF_PhongToMetalRoughness")
+PackageRedirects=(OldName="/Interchange/Functions/MF_SchlickApprox", NewName="/InterchangeAssets/Functions/MF_SchlickApprox")
+PackageRedirects=(OldName="/Interchange/Functions/MX_AbsorptionVDF", NewName="/InterchangeAssets/Functions/MX_AbsorptionVDF")

+PackageRedirects=(OldName="/Interchange/Functions/MX_AnisotropicVDF", NewName="/InterchangeAssets/Functions/MX_AnisotropicVDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Artistic_IOR", NewName="/InterchangeAssets/Functions/MX_Artistic_IOR")
+PackageRedirects=(OldName="/Interchange/Functions/MX_BurleyDiffuseBSDF", NewName="/InterchangeAssets/Functions/MX_BurleyDiffuseBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_ConductorBSDF", NewName="/InterchangeAssets/Functions/MX_ConductorBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_ConicalEDF", NewName="/InterchangeAssets/Functions/MX_ConicalEDF")

+PackageRedirects=(OldName="/Interchange/Functions/MX_DielectricBSDF", NewName="/InterchangeAssets/Functions/MX_DielectricBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_GeneralizedSchlickBSDF", NewName="/InterchangeAssets/Functions/MX_GeneralizedSchlickBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_MeasuredEDF", NewName="/InterchangeAssets/Functions/MX_MeasuredEDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_OpenPBR_Opaque", NewName="/InterchangeAssets/Functions/MX_OpenPBR_Opaque")
+PackageRedirects=(OldName="/Interchange/Functions/MX_OpenPBR_Translucent", NewName="/InterchangeAssets/Functions/MX_OpenPBR_Translucent")

+PackageRedirects=(OldName="/Interchange/Functions/MX_OrenNayarBSDF", NewName="/InterchangeAssets/Functions/MX_OrenNayarBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Place2D", NewName="/InterchangeAssets/Functions/MX_Place2D")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Roughness_Anisotropy", NewName="/InterchangeAssets/Functions/MX_Roughness_Anisotropy")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Roughness_Dual", NewName="/InterchangeAssets/Functions/MX_Roughness_Dual")
+PackageRedirects=(OldName="/Interchange/Functions/MX_SheenBSDF", NewName="/InterchangeAssets/Functions/MX_SheenBSDF")

+PackageRedirects=(OldName="/Interchange/Functions/MX_StandardSurface", NewName="/InterchangeAssets/Functions/MX_StandardSurface")
+PackageRedirects=(OldName="/Interchange/Functions/MX_SubsurfaceBSDF", NewName="/InterchangeAssets/Functions/MX_SubsurfaceBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Surface", NewName="/InterchangeAssets/Functions/MX_Surface")
+PackageRedirects=(OldName="/Interchange/Functions/MX_SurfaceUnlit", NewName="/InterchangeAssets/Functions/MX_SurfaceUnlit")
+PackageRedirects=(OldName="/Interchange/Functions/MX_ThinFilmBSDF", NewName="/InterchangeAssets/Functions/MX_ThinFilmBSDF")

+PackageRedirects=(OldName="/Interchange/Functions/MX_TranslucentBSDF", NewName="/InterchangeAssets/Functions/MX_TranslucentBSDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_Transmission", NewName="/InterchangeAssets/Functions/MX_Transmission")
+PackageRedirects=(OldName="/Interchange/Functions/MX_TransmissionSurface", NewName="/InterchangeAssets/Functions/MX_TransmissionSurface")
+PackageRedirects=(OldName="/Interchange/Functions/MX_UniformEDF", NewName="/InterchangeAssets/Functions/MX_UniformEDF")
+PackageRedirects=(OldName="/Interchange/Functions/MX_UsdPreviewSurface", NewName="/InterchangeAssets/Functions/MX_UsdPreviewSurface")


; Redirectors for /Interchange/gltf/ (8)
+PackageRedirects=(OldName="/Interchange/gltf/M_ClearCoat", NewName="/InterchangeAssets/gltf/M_ClearCoat")
+PackageRedirects=(OldName="/Interchange/gltf/M_Default", NewName="/InterchangeAssets/gltf/M_Default")
+PackageRedirects=(OldName="/Interchange/gltf/M_Sheen", NewName="/InterchangeAssets/gltf/M_Sheen")
+PackageRedirects=(OldName="/Interchange/gltf/M_SpecularGlossiness", NewName="/InterchangeAssets/gltf/M_SpecularGlossiness")
+PackageRedirects=(OldName="/Interchange/gltf/M_Transmission", NewName="/InterchangeAssets/gltf/M_Transmission")

+PackageRedirects=(OldName="/Interchange/gltf/M_Unlit", NewName="/InterchangeAssets/gltf/M_Unlit")
+PackageRedirects=(OldName="/Interchange/gltf/PerceivedBrightness", NewName="/InterchangeAssets/gltf/PerceivedBrightness")
+PackageRedirects=(OldName="/Interchange/gltf/SpecGlossToMetalRoughness", NewName="/InterchangeAssets/gltf/SpecGlossToMetalRoughness")


; Redirectors for /Interchange/gltf/MaterialBodies (6)
+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_ClearCoat_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_ClearCoat_Body")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_Default_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_Default_Body")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_Sheen_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_Sheen_Body")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_SpecularGlossiness_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_SpecularGlossiness_Body")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_Transmission_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_Transmission_Body")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialBodies/MF_Unlit_Body", NewName="/InterchangeAssets/gltf/MaterialBodies/MF_Unlit_Body")


; Redirectors for /Interchange/gltf/MaterialFunctions (23)
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_BaseColor", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_BaseColor")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Clearcoat", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Clearcoat")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_DiffuseSpecGloss", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_DiffuseSpecGloss")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Emissive", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Emissive")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Fresnel_DS", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Fresnel_DS")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_IOR", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_IOR")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Iridescence", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Iridescence")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_MaxComponentValue", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_MaxComponentValue")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_MetallicRoughness", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_MetallicRoughness")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Normals", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Normals")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Occlusion", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Occlusion")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_PerceivedBrightness", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_PerceivedBrightness")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_RotateNormals_TS", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_RotateNormals_TS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_RotateV2", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_RotateV2")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_ScaleNormal", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_ScaleNormal")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Sheen", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Sheen")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_SpecGlossToMetalRoughness", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_SpecGlossToMetalRoughness")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Specular", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Specular")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_SpecularAnisotropy", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_SpecularAnisotropy")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Temporal_Blur", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Temporal_Blur")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_TransformUVs", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_TransformUVs")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_Transmission", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_Transmission")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialFunctions/MF_TransmissionOpacity", NewName="/InterchangeAssets/gltf/MaterialFunctions/MF_TransmissionOpacity")


; Redirectors for /Interchange/gltf/MaterialInstances (32)
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Blend", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Blend")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Blend_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Blend_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Mask", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Mask")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Mask_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Mask_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Opaque", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Opaque")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_ClearCoat_Opaque_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Opaque_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Blend", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Blend")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Blend_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Blend_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Mask", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Mask")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Mask_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Mask_DS")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Opaque", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Opaque")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Default_Opaque_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Default_Opaque_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Blend", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Blend")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Blend_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Blend_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Mask", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Mask")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Mask_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Mask_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Opaque", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Opaque")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Sheen_Opaque_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Opaque_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Blend", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Blend")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Blend_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Blend_DS")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Mask", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Mask")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Mask_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Mask_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Transmission", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Transmission")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Transmission_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Transmission_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Blend", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Blend")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Blend_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Blend_DS")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Mask", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Mask")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Mask_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Mask_DS")

+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Opaque", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Opaque")
+PackageRedirects=(OldName="/Interchange/gltf/MaterialInstances/MI_Unlit_Opaque_DS", NewName="/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Opaque_DS")


; Redirectors for /Interchange/gltf/Textures (6)
+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_Anisotropy_Direction_Linear", NewName="/InterchangeAssets/gltf/Textures/T_Anisotropy_Direction_Linear")
+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_Black_srgb", NewName="/InterchangeAssets/gltf/Textures/T_Black_srgb")
+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_Generic_N", NewName="/InterchangeAssets/gltf/Textures/T_Generic_N")
+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_Gray_Linear", NewName="/InterchangeAssets/gltf/Textures/T_Gray_Linear")
+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_White_Linear", NewName="/InterchangeAssets/gltf/Textures/T_White_Linear")

+PackageRedirects=(OldName="/Interchange/gltf/Textures/T_White_srgb", NewName="/InterchangeAssets/gltf/Textures/T_White_srgb")


; Redirectors for /Interchange/Materials (21)
+PackageRedirects=(OldName="/Interchange/Materials/ClearCoatMaterial_MR", NewName="/InterchangeAssets/Materials/ClearCoatMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/ClearCoatMaterial_SG", NewName="/InterchangeAssets/Materials/ClearCoatMaterial_SG")
+PackageRedirects=(OldName="/Interchange/Materials/CommonMaterial", NewName="/InterchangeAssets/Materials/CommonMaterial")
+PackageRedirects=(OldName="/Interchange/Materials/DecalMaterial", NewName="/InterchangeAssets/Materials/DecalMaterial")
+PackageRedirects=(OldName="/Interchange/Materials/LambertSurfaceMaterial", NewName="/InterchangeAssets/Materials/LambertSurfaceMaterial")

+PackageRedirects=(OldName="/Interchange/Materials/MapOrColorFunction", NewName="/InterchangeAssets/Materials/MapOrColorFunction")
+PackageRedirects=(OldName="/Interchange/Materials/MapOrLinearColorFunction", NewName="/InterchangeAssets/Materials/MapOrLinearColorFunction")
+PackageRedirects=(OldName="/Interchange/Materials/MapOrScalarFunction", NewName="/InterchangeAssets/Materials/MapOrScalarFunction")
+PackageRedirects=(OldName="/Interchange/Materials/OrenNayerMaterial", NewName="/InterchangeAssets/Materials/OrenNayerMaterial")
+PackageRedirects=(OldName="/Interchange/Materials/PBRSurfaceFunction_MR", NewName="/InterchangeAssets/Materials/PBRSurfaceFunction_MR")

+PackageRedirects=(OldName="/Interchange/Materials/PBRSurfaceFunction_SG", NewName="/InterchangeAssets/Materials/PBRSurfaceFunction_SG")
+PackageRedirects=(OldName="/Interchange/Materials/PBRSurfaceMaterial_MRPBRSurfaceMaterial_MR", NewName="/InterchangeAssets/Materials/PBRSurfaceMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/PBRSurfaceMaterial_SG", NewName="/InterchangeAssets/Materials/PBRSurfaceMaterial_SG")
+PackageRedirects=(OldName="/Interchange/Materials/PhongSurfaceMaterial", NewName="/InterchangeAssets/Materials/PhongSurfaceMaterial")
+PackageRedirects=(OldName="/Interchange/Materials/SheenMaterial_MR", NewName="/InterchangeAssets/Materials/SheenMaterial_MR")

+PackageRedirects=(OldName="/Interchange/Materials/SheenMaterial_SG", NewName="/InterchangeAssets/Materials/SheenMaterial_SG")
+PackageRedirects=(OldName="/Interchange/Materials/SubsurfaceMaterial_MR", NewName="/InterchangeAssets/Materials/SubsurfaceMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/SubsurfaceMaterial_SG", NewName="/InterchangeAssets/Materials/SubsurfaceMaterial_SG")
+PackageRedirects=(OldName="/Interchange/Materials/ThinTranslucentMaterial_MR", NewName="/InterchangeAssets/Materials/ThinTranslucentMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/ThinTranslucentMaterial_SG", NewName="/InterchangeAssets/Materials/ThinTranslucentMaterial_SG")

+PackageRedirects=(OldName="/Interchange/Materials/UnlitMaterial", NewName="/InterchangeAssets/Materials/UnlitMaterial")


; Redirectors for /Interchange/Substrate (17)
+PackageRedirects=(OldName="/Interchange/Substrate/MX_AbsorptionVDF", NewName="/InterchangeAssets/Substrate/MX_AbsorptionVDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_AnisotropicVDF", NewName="/InterchangeAssets/Substrate/MX_AnisotropicVDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_BurleyDiffuseBSDF", NewName="/InterchangeAssets/Substrate/MX_BurleyDiffuseBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_ConductorBSDF", NewName="/InterchangeAssets/Substrate/MX_ConductorBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_ConicalEDF", NewName="/InterchangeAssets/Substrate/MX_ConicalEDF")

+PackageRedirects=(OldName="/Interchange/Substrate/MX_DielectricBSDF", NewName="/InterchangeAssets/Substrate/MX_DielectricBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_GeneralizedSchlickBSDF", NewName="/InterchangeAssets/Substrate/MX_GeneralizedSchlickBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_MeasuredEDF", NewName="/InterchangeAssets/Substrate/MX_MeasuredEDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_OrenNayarBSDF", NewName="/InterchangeAssets/Substrate/MX_OrenNayarBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_SheenBSDF", NewName="/InterchangeAssets/Substrate/MX_SheenBSDF")

+PackageRedirects=(OldName="/Interchange/Substrate/MX_SubsurfaceBSDF", NewName="/InterchangeAssets/Substrate/MX_SubsurfaceBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_Surface", NewName="/InterchangeAssets/Substrate/MX_Surface")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_SurfaceUnlit", NewName="/InterchangeAssets/Substrate/MX_SurfaceUnlit")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_ThinFilmBSDF", NewName="/InterchangeAssets/Substrate/MX_ThinFilmBSDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_TranslucentBSDF", NewName="/InterchangeAssets/Substrate/MX_TranslucentBSDF")

+PackageRedirects=(OldName="/Interchange/Substrate/MX_UniformEDF", NewName="/InterchangeAssets/Substrate/MX_UniformEDF")
+PackageRedirects=(OldName="/Interchange/Substrate/MX_UsdPreviewSurface", NewName="/InterchangeAssets/Substrate/MX_UsdPreviewSurface")


; Redirectors for /Interchange/Utilities (2)
+PackageRedirects=(OldName="/Interchange/Utilities/New_LUT", NewName="/InterchangeAssets/Utilities/New_LUT")
+PackageRedirects=(OldName="/Interchange/Utilities/T_Bayer64_Grayscale_64px", NewName="/InterchangeAssets/Utilities/T_Bayer64_Grayscale_64px")
