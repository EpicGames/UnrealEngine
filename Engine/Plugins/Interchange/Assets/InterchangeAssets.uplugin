{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange Framework Assets",
	"Description": "The Interchange Framework Assets plugin exposes the assets used by the Interchange import framework.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"IsExperimentalVersion": false,
	"Modules": [
		{
			"Name": "InterchangeAssets",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "BaseMaterial",
			"Enabled": true
		}
	]
}