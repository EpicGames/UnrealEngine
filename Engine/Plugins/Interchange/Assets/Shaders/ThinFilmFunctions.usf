// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float3 ThinFilmEvalSensitivityA(float opd, float shift)
{
	// Use Gaussian fits, given by 3 parameters: val, pos and var
	float phase = 2 * PI * opd * 1.0e-6;
	float3 val = float3(5.4856e-13, 4.4201e-13, 5.2481e-13);
	float3 pos = float3(1.6810e+06, 1.7953e+06, 2.2084e+06);
	float3 var = float3(4.3278e+09, 9.3046e+09, 6.6121e+09);
	float3 xyz = val * sqrt(2 * PI * var) * cos(pos * phase + shift) * exp(-var * phase * phase);
	xyz.x += 9.7470e-14 * sqrt(2 * PI * 4.5282e+09) * cos(2.2399e+06 * phase + shift) * exp(-4.5282e+09 * phase * phase);
	return xyz / 1.0685e-7;
}

void ThinFilmFresnelDielectricA(
	in float ct1,
	in float n1,
	in float n2,
	out float2 R,
	out float2 phi)
{
	float st1 = (1 - ct1 * ct1); // Sinus theta1 'squared'
	float nr = n1 / n2;

	if (Pow2(nr) * st1 > 1)  // Total reflection
	{
		R = float2(1, 1);
		phi = 2.0 * atan(float2(
			-Pow2(nr) * sqrt(st1 - 1.0 / Pow2(nr)) / ct1,
			-sqrt(st1 - 1.0 / Pow2(nr)) / ct1));
	}
	else    // Transmission & Reflection
	{

		float ct2 = sqrt(1 - Pow2(nr) * st1);
		float2 r = float2(
			(n2 * ct1 - n1 * ct2) / (n2 * ct1 + n1 * ct2),
			(n1 * ct1 - n2 * ct2) / (n1 * ct1 + n2 * ct2));
		phi.x = (r.x < 0.0) ? PI : 0.0;
		phi.y = (r.y < 0.0) ? PI : 0.0;
		R = Pow2(r);
	}
}

void ThinFilmFresnelConductorA(
	in float ct1,
	in float n1,
	in float n2,
	in float k,
	out float2 R,
	out float2 phi)
{
	if (k == 0)
	{ // use dielectric formula to avoid numerical issues
		ThinFilmFresnelDielectricA(ct1, n1, n2, R, phi);
	}
	else
	{
		float A = Pow2(n2) * (1 - Pow2(k)) - Pow2(n1) * (1 - Pow2(ct1));
		float B = sqrt(Pow2(A) + Pow2(2 * Pow2(n2) * k));
		float U = sqrt((A + B) / 2.0);
		float V = sqrt((B - A) / 2.0);

		R.y = (Pow2(n1 * ct1 - U) + Pow2(V)) / (Pow2(n1 * ct1 + U) + Pow2(V));
		phi.y = atan2(2 * n1 * V * ct1, Pow2(U) + Pow2(V) - Pow2(n1 * ct1)) + PI;

		R.x = (Pow2(Pow2(n2) * (1 - Pow2(k)) * ct1 - n1 * U) + Pow2(2 * Pow2(n2) * k * ct1 - n1 * V)) / (Pow2(Pow2(n2) * (1 - Pow2(k)) * ct1 + n1 * U) + Pow2(2 * Pow2(n2) * k * ct1 + n1 * V));
		phi.x = atan2(2 * n1 * Pow2(n2) * ct1 * (2 * k * U - (1 - Pow2(k)) * V), Pow2(Pow2(n2) * (1 + Pow2(k)) * ct1) - Pow2(n1) * (Pow2(U) + Pow2(V)));
	}
}

// [Gulbrandsen, "Artist Friendly Metallic Fresnel"]
// Compute complex index of refraction N = n + ik from F0 and edge tint.
void ComputeComplexIORFromF0AndEdgeTintA(
    in float3 Reflectivity, 
    in float3 EdgeTint, 
    inout float3 n, 
    inout float3 k)
{
	const float3 g = EdgeTint;
	const float3 r = clamp(Reflectivity, 0.0, 0.999);
	const float3 SqrtR = sqrt(r);

	n = g * (1 - r) / (1 + r) + (1 - g) * (1 + SqrtR) / (1 - SqrtR);

	const float3 k2 = ((n + 1) * (n + 1) * r - (n - 1) * (n - 1)) / (1 - r);
	k = sqrt(k2);
}

float ThinFilmDepolA(float2 polV) { return 0.5 * (polV.x + polV.y); }
float3 ThinFilmDepolColorA(float3 colS, float3 colP) { return 0.5 * (colS + colP); }

float3 SchlickB(float3 F0, float3 F90, float VoH)
{
	float Fc = Pow5(1 - VoH);
	return F90 * Fc + (1 - Fc) * F0;
}

float3 SchlickA( float3 SpecularColor, float VoH )
{
	float Fc = Pow5( 1 - VoH );					// 1 sub, 3 mul
	//return Fc + (1 - Fc) * SpecularColor;		// 1 add, 3 mad
	
	// Anything less than 2% is physically impossible and is instead considered to be shadowing
	return saturate( 50.0 * SpecularColor.g ) * Fc + (1 - Fc) * SpecularColor;
}

float DielectricToF0A(float Ior)
{
	const float F0Sqrt = (Ior-1)/(Ior+1);
	const float F0 = F0Sqrt*F0Sqrt;
	return F0;
}