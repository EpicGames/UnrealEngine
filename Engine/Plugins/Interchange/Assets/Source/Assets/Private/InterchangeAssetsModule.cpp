// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

class FInterchangeAssetsModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("InterchangeAssets"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/InterchangeAssets"), PluginShaderDir);
	}

	virtual void ShutdownModule() override
	{}
};

IMPLEMENT_MODULE(FInterchangeAssetsModule, InterchangeAssets)

